{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design Data Link OAuth",
	"Description": "Motion Design Data Link functionality for OAuth 2.0",
	"Category": "Other",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DataLinkOAuth",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataLinkOAuthEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DataLink",
			"Enabled": true
		}
	]
}