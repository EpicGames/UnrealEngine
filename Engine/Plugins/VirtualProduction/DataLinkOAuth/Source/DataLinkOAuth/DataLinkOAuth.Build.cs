// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLinkOAuth : ModuleRules
{
	public DataLinkOAuth(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DataLink",
				"DataLinkHttp",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"HTTP",
				"HTTPServer",
				"Json",
				"Sockets",
			}
		);
	}
}
