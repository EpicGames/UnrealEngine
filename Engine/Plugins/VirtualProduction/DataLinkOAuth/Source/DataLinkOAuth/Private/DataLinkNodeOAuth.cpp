// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkNodeOAuth.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkExecutor.h"
#include "DataLinkHttpSettings.h"
#include "DataLinkNames.h"
#include "DataLinkOAuthAuthRequest.h"
#include "DataLinkOAuthCodeAccessExchange.h"
#include "DataLinkOAuthHandle.h"
#include "DataLinkOAuthInstance.h"
#include "DataLinkOAuthLog.h"
#include "DataLinkOAuthSettings.h"
#include "DataLinkOAuthSubsystem.h"
#include "DataLinkOAuthToken.h"
#include "DataLinkPinBuilder.h"
#include "HttpServerRequest.h"

#define LOCTEXT_NAMESPACE "DataLinkNodeOAuth"

UDataLinkNodeOAuth::UDataLinkNodeOAuth()
{
	InstanceStruct = FDataLinkNodeOAuthInstance::StaticStruct();
}

void UDataLinkNodeOAuth::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Inputs.Add(UE::DataLinkOAuth::InputHttp)
		.SetDisplayName(LOCTEXT("HttpSettingsDisplay", "Http Settings"))
		.SetStruct<FDataLinkHttpSettings>();

	Inputs.Add(UE::DataLinkOAuth::InputOAuth)
		.SetDisplayName(LOCTEXT("OAuthSettingsDisplay", "OAuth Settings"))
		.SetStruct<FDataLinkOAuthSettingsWrapper>();

	Outputs.Add(UE::DataLink::OutputDefault)
		.SetDisplayName(LOCTEXT("OutputDisplay", "Http Settings"))
		.SetStruct<FDataLinkHttpSettings>();
}

EDataLinkExecutionReply UDataLinkNodeOAuth::OnExecute(FDataLinkExecutor& InExecutor) const
{
	FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstanceMutable(this);

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();
	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	UDataLinkOAuthSettings* const OAuthSettings = InputDataViewer.Get<FDataLinkOAuthSettingsWrapper>(UE::DataLinkOAuth::InputOAuth).OAuthSettings;
	if (!OAuthSettings)
	{
		UE_LOGF(LogDataLinkOAuth, Error, "[%ls] Data Link OAuth - Invalid OAuth settings.", InExecutor.GetContextName().GetData());
		return EDataLinkExecutionReply::Unhandled;
	}

	// Attempt to re-use an existing access token if available and not expired.
	if (UDataLinkOAuthSubsystem* OAuthSubsystem = UDataLinkOAuthSubsystem::Get())
	{
		OAuthSubsystem->CleanExpiredTokens();

		if (const FDataLinkOAuthToken* OAuthToken = OAuthSubsystem->FindToken(OAuthSettings))
		{
			// Copy Input Http Settings into Output
			FDataLinkHttpSettings& OutputHttpSettings = OutputDataViewer.Get<FDataLinkHttpSettings>(UE::DataLink::OutputDefault);
			OutputHttpSettings = InputDataViewer.Get<FDataLinkHttpSettings>(UE::DataLinkOAuth::InputHttp);

			if (OAuthSettings->AuthorizeHttpRequest(*OAuthToken, OutputHttpSettings))
			{
				InExecutor.Next(this);
				return EDataLinkExecutionReply::Handled;
			}
		}
	}

	FDataLinkNodeOAuthInstance& OAuthInstance = NodeInstance.GetInstanceDataMutable().Get<FDataLinkNodeOAuthInstance>();
	OAuthInstance.SharedData = OAuthSettings->MakeSharedData();

	UE::DataLinkOAuth::FAuthRequestParams AuthRequestParams;
	AuthRequestParams.OAuthSettings = OAuthSettings;
	AuthRequestParams.OAuthInstanceView = OAuthInstance;
	AuthRequestParams.OnAuthResponse.BindUObject(this, &UDataLinkNodeOAuth::OnAuthResponse, InExecutor.AsWeak());

	if (!UE::DataLinkOAuth::RequestAuthorization(AuthRequestParams))
	{
		UE_LOGF(LogDataLinkOAuth, Error, "[%ls] Data Link OAuth - Authorization request failed.", InExecutor.GetContextName().GetData());
		return EDataLinkExecutionReply::Unhandled;
	}

	return EDataLinkExecutionReply::Handled;
}

void UDataLinkNodeOAuth::OnAuthResponse(const FHttpServerRequest& InRequest, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const
{
	TSharedPtr<FDataLinkExecutor> Executor = InExecutorWeak.Pin();
	if (!Executor.IsValid())
	{
		return;
	}

	FDataLinkNodeInstance& NodeInstance = Executor->GetNodeInstanceMutable(this);

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();

	UDataLinkOAuthSettings* const OAuthSettings = InputDataViewer.Get<FDataLinkOAuthSettingsWrapper>(UE::DataLinkOAuth::InputOAuth).OAuthSettings;
	check(OAuthSettings);

	FDataLinkNodeOAuthInstance& OAuthInstance = NodeInstance.GetInstanceDataMutable().Get<FDataLinkNodeOAuthInstance>();
	if (!OAuthSettings->ValidateRequest(InRequest, OAuthInstance))
	{
		// Request did not come from this OAuth instance. Skip
		// NOTE: FailNode isn't called here because the expected request might still come
		return;
	}

	// Stop listening as the expected request was found
	OAuthInstance.StopListening();

	FStringView AuthCodeView;
	if (!OAuthSettings->FindAuthCode(InRequest, OAuthInstance, AuthCodeView) || AuthCodeView.IsEmpty())
	{
		UE_LOGF(LogDataLinkOAuth, Error, "[%ls] Data Link OAuth - Invalid Authorization Code (Settings: %ls)"
			, Executor->GetContextName().GetData()
			, *OAuthSettings->GetName());

		Executor->Fail(this);
		return;
	}

	UE::DataLinkOAuth::FExchangeAuthCodeParams ExchangeAuthCodeParams;
	ExchangeAuthCodeParams.OAuthSettings = OAuthSettings;
	ExchangeAuthCodeParams.AuthCodeView = AuthCodeView;
	ExchangeAuthCodeParams.OAuthInstanceView = OAuthInstance;
	ExchangeAuthCodeParams.OnResponse.BindUObject(this, &UDataLinkNodeOAuth::OnExchangeCodeResponse, InExecutorWeak);

	if (!UE::DataLinkOAuth::ExchangeAuthCodeForAccess(ExchangeAuthCodeParams))
	{
		UE_LOGF(LogDataLinkOAuth, Error, "[%ls] Data Link OAuth - Failed to Exchange Auth Code For Access (Settings: %ls)"
			, Executor->GetContextName().GetData()
			, *OAuthSettings->GetName());

		Executor->Fail(this);
	}
}

void UDataLinkNodeOAuth::OnExchangeCodeResponse(FStringView InResponse, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const
{
	TSharedPtr<FDataLinkExecutor> Executor = InExecutorWeak.Pin();
	if (!Executor.IsValid())
	{
		return;
	}

	const FDataLinkNodeInstance& NodeInstance = Executor->GetNodeInstance(this);

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();

	UDataLinkOAuthSettings* const OAuthSettings = InputDataViewer.Get<FDataLinkOAuthSettingsWrapper>(UE::DataLinkOAuth::InputOAuth).OAuthSettings;
	check(OAuthSettings);

	if (InResponse.IsEmpty())
	{
		UE_LOGF(LogDataLinkOAuth, Error, "[%ls] Data Link OAuth - Failed to get a valid response with Access Token (Settings: %ls)"
			, Executor->GetContextName().GetData()
			, *OAuthSettings->GetName());

		Executor->Fail(this);
		return;
	}

	FDataLinkOAuthToken OAuthToken;
	if (!OAuthSettings->BuildAuthToken(InResponse, OAuthToken))
	{
		UE_LOGF(LogDataLinkOAuth, Error, "[%ls] Data Link OAuth - Failed to build OAuth Token. (Settings: %ls) Exchange Code Response: %ls"
			, Executor->GetContextName().GetData()
			, *OAuthSettings->GetName()
			, InResponse.GetData());

		Executor->Fail(this);
		return;
	}

	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	// Copy Input Http Settings into Output
	FDataLinkHttpSettings& OutputHttpSettings = OutputDataViewer.Get<FDataLinkHttpSettings>(UE::DataLink::OutputDefault);
	OutputHttpSettings = InputDataViewer.Get<FDataLinkHttpSettings>(UE::DataLinkOAuth::InputHttp);

	if (!OAuthSettings->AuthorizeHttpRequest(OAuthToken, OutputHttpSettings))
	{
		UE_LOGF(LogDataLinkOAuth, Error, "[%ls] Data Link OAuth - OAuth Token unexpectedly failed to authorize the Http Request. (Settings: %ls) Exchange Code Response: %ls"
			, Executor->GetContextName().GetData()
			, *OAuthSettings->GetName()
			, InResponse.GetData());

		Executor->Fail(this);
		return;
	}

	if (UDataLinkOAuthSubsystem* OAuthSubsystem = UDataLinkOAuthSubsystem::Get())
	{
		OAuthSubsystem->RegisterToken(OAuthSettings, OAuthToken);
	}

	Executor->Next(this);
}

#undef LOCTEXT_NAMESPACE
