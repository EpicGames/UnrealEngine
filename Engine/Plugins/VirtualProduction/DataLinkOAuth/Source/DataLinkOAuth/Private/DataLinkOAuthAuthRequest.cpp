// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthAuthRequest.h"
#include "DataLinkNodeOAuth.h"
#include "DataLinkOAuthHandle.h"
#include "DataLinkOAuthInstance.h"
#include "DataLinkOAuthLog.h"
#include "DataLinkOAuthSettings.h"
#include "DataLinkOAuthSubsystem.h"
#include "HttpServerModule.h"
#include "IHttpRouter.h"
#include "SocketSubsystem.h"
#include "Sockets.h"

namespace UE::DataLinkOAuth::Private
{
	TUniquePtr<FHttpServerResponse> CreateSuccessResponse()
	{
		constexpr const TCHAR* ResponseString = TEXT("<html>"\
			"<head><meta http-equiv='refresh' content='3;url=http://unrealengine.com/'>""</head>"\
			"<body>Please return to the App.</body>"\
			"</html>");

		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Code = EHttpServerResponseCodes::Ok;

		const FTCHARToUTF8 ConvertToUtf8(ResponseString);
		Response->Body.Append(reinterpret_cast<const uint8*>(ConvertToUtf8.Get()), ConvertToUtf8.Length());
		return Response;
	}

	bool FindUnusedPort(int32& OutPort)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		if (!SocketSubsystem)
		{
			UE_LOGF(LogDataLinkOAuth, Error, "Find Unused Port failed. Invalid Socket Subsystem");
			return false;
		}

		// Create address with port 0. After socket has been bound FSocket::GetAddress should give the actual port assigned
		TSharedRef<FInternetAddr> InternetAddress = SocketSubsystem->CreateInternetAddr();
		InternetAddress->SetLoopbackAddress();
		InternetAddress->SetPort(0);

		constexpr const TCHAR* SocketDescription = TEXT("Data Link OAuth - FindUnusedPort");
		FSocket* Socket = SocketSubsystem->CreateSocket(NAME_Stream, SocketDescription, InternetAddress->GetProtocolType());
		if (!Socket)
		{
			UE_LOGF(LogDataLinkOAuth, Error, "Find Unused Port failed. Could not create Socket [%ls, %ls]"
				, SocketDescription
				, *InternetAddress->ToString(/*bAppendPort*/false));
			return false;
		}

		if (!Socket->Bind(*InternetAddress))
		{
			UE_LOGF(LogDataLinkOAuth, Error, "Find Unused Port failed. Failed to Bind Socket [%ls, %ls]"
				, SocketDescription
				, *InternetAddress->ToString(/*bAppendPort*/false));
			return false;
		}

		// Re-use InternetAddress to get the port
		Socket->GetAddress(*InternetAddress);
		OutPort = InternetAddress->GetPort();
		Socket->Close();
		return true;
	}

	FDataLinkOAuthHandle StartListening(int32 InListenPort, const FOnAuthResponse& InOnAuthResponse)
	{
		const TSharedPtr<IHttpRouter> Router = FHttpServerModule::Get().GetHttpRouter(InListenPort, /*bFailOnBindFailure*/true);
		if (!Router.IsValid())
		{
			UE_LOGF(LogDataLinkOAuth, Error, "Start Listening failed. Could not get HTTP Router for port %d.", InListenPort);
			return FDataLinkOAuthHandle();
		}

		UDataLinkOAuthSubsystem::FListenInstance ListenInstance;
		ListenInstance.RouterWeak = Router;
		ListenInstance.RequestPreprocessorHandle = Router->RegisterRequestPreprocessor(FHttpRequestHandler::CreateLambda(
			[OnAuthResponse = InOnAuthResponse](const FHttpServerRequest& InRequest, const FHttpResultCallback& InOnComplete)->bool
			{
				InOnComplete(CreateSuccessResponse());
				OnAuthResponse.ExecuteIfBound(InRequest);
				return true;
			}));

		return UDataLinkOAuthSubsystem::Get()->RegisterListenInstance(MoveTemp(ListenInstance));
	}
}

bool UE::DataLinkOAuth::RequestAuthorization(const FAuthRequestParams& InParams)
{
	UDataLinkOAuthSubsystem* OAuthSubsystem = UDataLinkOAuthSubsystem::Get();
	if (!OAuthSubsystem)
	{
		UE_LOGF(LogDataLinkOAuth, Error, "Start Listening failed. Invalid OAuth Subsystem.");
		return false;
	}

	int32 ListenPort;
	if (! Private::FindUnusedPort(ListenPort))
	{
		UE_LOGF(LogDataLinkOAuth, Error, "Start Listening failed. Could not find unused port.");
		return false;
	}

	const FDataLinkOAuthHandle ListenHandle = Private::StartListening(ListenPort, InParams.OnAuthResponse);
	if (!ListenHandle.IsValid())
	{
		return false;
	}

	FDataLinkNodeOAuthInstance& OAuthInstance = InParams.OAuthInstanceView.Get();
	OAuthInstance.ListenHandle = ListenHandle;
	OAuthInstance.ListenPort = ListenPort;

	UDataLinkOAuthSettings::FUrlBuilder RequestUrl;
	check(InParams.OAuthSettings);
	InParams.OAuthSettings->BuildAuthRequestUrl(RequestUrl, OAuthInstance);

	FHttpServerModule::Get().StartAllListeners();

	FPlatformProcess::LaunchURL(*RequestUrl, nullptr, nullptr);
	return true;
}
