// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "StructUtils/StructView.h"

class FDataLinkOAuthHandle;
class UDataLinkOAuthSettings;
struct FDataLinkNodeOAuthInstance;
struct FHttpServerRequest;

namespace UE::DataLinkOAuth
{
	using FOnAuthResponse = TDelegate<void(const FHttpServerRequest&)>;

	struct FAuthRequestParams
	{
		/** Settings to used for Authorization */
		UDataLinkOAuthSettings* OAuthSettings;

		/** OAuth Instance Data to Read from and Write to */
		TStructView<FDataLinkNodeOAuthInstance> OAuthInstanceView;

		/** Called when the Authorization has been granted*/
		FOnAuthResponse OnAuthResponse;
	};

	bool RequestAuthorization(const FAuthRequestParams& InParams);
}
