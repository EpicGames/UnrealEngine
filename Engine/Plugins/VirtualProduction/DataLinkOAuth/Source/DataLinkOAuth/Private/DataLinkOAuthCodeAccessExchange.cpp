// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthCodeAccessExchange.h"
#include "DataLinkNodeOAuth.h"
#include "DataLinkOAuthInstance.h"
#include "DataLinkOAuthLog.h"
#include "DataLinkOAuthSettings.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"

namespace UE::DataLinkOAuth::Private
{
	FString GetResponseString(FHttpResponsePtr InResponse, bool bInProcessedSuccessfully)
	{
		if (!bInProcessedSuccessfully || !InResponse.IsValid())
		{
			UE_LOGF(LogDataLinkOAuth, Error, "OAuth Code Exchange failed to be processed.");
			return FString();
		}

		const int32 ResponseCode = InResponse->GetResponseCode();
		if (!EHttpResponseCodes::IsOk(ResponseCode))
		{
			UE_LOGF(LogDataLinkOAuth, Error, "OAuth Code Exchange failed with response code %d", ResponseCode);
			return FString();
		}

		return InResponse->GetContentAsString();
	}
}

bool UE::DataLinkOAuth::ExchangeAuthCodeForAccess(const FExchangeAuthCodeParams& InParams)
{
	if (InParams.AuthCodeView.IsEmpty())
	{
		UE_LOGF(LogDataLinkOAuth, Error, "OAuth Code Exchange Failed. Invalid Auth Code.");
		return false;
	}

	const TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();

	FDataLinkNodeOAuthInstance& OAuthInstance = InParams.OAuthInstanceView.Get();

	UDataLinkOAuthSettings::FUrlBuilder RequestUrl;
	check(InParams.OAuthSettings);

	if (!InParams.OAuthSettings->BuildExchangeCodeTokenUrl(RequestUrl, OAuthInstance, InParams.AuthCodeView))
	{
		UE_LOGF(LogDataLinkOAuth, Error, "OAuth Code Exchange Failed. Could not build request URL. (Settings: %ls)"
			, *InParams.OAuthSettings->GetName());
		return false;
	}

	HttpRequest->SetVerb(TEXT("POST"));
	HttpRequest->SetURL(*RequestUrl);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/x-www-form-urlencoded"));
	HttpRequest->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnGameThread);

	HttpRequest->OnProcessRequestComplete().BindLambda(
		[OnResponse = InParams.OnResponse](FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bInProcessedSuccessfully)
		{
			const FString ResponseString = Private::GetResponseString(InResponse, bInProcessedSuccessfully);
			OnResponse.ExecuteIfBound(ResponseString);
		});

	return HttpRequest->ProcessRequest();
}
