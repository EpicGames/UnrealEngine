// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Delegates/Delegate.h"
#include "StructUtils/StructView.h"

class UDataLinkOAuthSettings;
struct FDataLinkNodeOAuthInstance;

namespace UE::DataLinkOAuth
{
	using FOnExchangeAuthCodeResponse = TDelegate<void(FStringView InResponseString)>;

	struct FExchangeAuthCodeParams
	{
		/** Settings to used for Authorization */
		UDataLinkOAuthSettings* OAuthSettings;

		/** OAuth Instance Data to Read from and Write to */
		TStructView<FDataLinkNodeOAuthInstance> OAuthInstanceView;

		/** Auth Code to exchange for Access Token */
		FStringView AuthCodeView;

		/** Called when the Exchange Request has received a response */
		FOnExchangeAuthCodeResponse OnResponse;
	};

	bool ExchangeAuthCodeForAccess(const FExchangeAuthCodeParams& InParams);
}
