// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthDefaultSettings.h"
#include "DataLinkOAuthDefaultSharedData.h"
#include "DataLinkOAuthInstance.h"
#include "DataLinkOAuthLog.h"
#include "HttpServerRequest.h"
#include "PlatformHttp.h"
#include "StructUtils/StructView.h"

UDataLinkOAuthDefaultSettings::UDataLinkOAuthDefaultSettings()
{
	SharedDataType = FDataLinkOAuthDefaultSharedData::StaticStruct();
}

bool UDataLinkOAuthDefaultSettings::BuildAuthRequestUrl(FUrlBuilder& OutRequestUrl, FDataLinkNodeOAuthInstance& InOAuthInstance) const
{
	FDataLinkOAuthDefaultSharedData& SharedData = InOAuthInstance.SharedData.GetMutable<FDataLinkOAuthDefaultSharedData>();

	const FString RedirectUriWithPort = FString::Printf(TEXT("%s:%s"), LoopbackAddress, *FString::FromInt(InOAuthInstance.ListenPort));

	OutRequestUrl << AuthorizationURL
		<< TEXT("?prompt=consent"\
			"&response_type=code"\
			"&access_type=offline")
		<< TEXT("&redirect_uri=") << FPlatformHttp::UrlEncode(RedirectUriWithPort)
		<< TEXT("&state=") << SharedData.State
		<< TEXT("&client_id=") << ClientId;

	if (!Scopes.IsEmpty())
	{
		OutRequestUrl << TEXT("&scope=");
		for (const FString& Scope : Scopes)
		{
			OutRequestUrl << FPlatformHttp::UrlEncode(Scope) << TEXT("+");
		}
		// Remove last '+'
		OutRequestUrl.RemoveAt(OutRequestUrl.Len() - 1, 1);
	}

	return true;
}

bool UDataLinkOAuthDefaultSettings::ValidateRequest(const FHttpServerRequest& InRequest, FDataLinkNodeOAuthInstance& InOAuthInstance) const
{
	FDataLinkOAuthDefaultSharedData& SharedData = InOAuthInstance.SharedData.GetMutable<FDataLinkOAuthDefaultSharedData>();

	// Proceed only if the states match, ensuring that this instance made the request
	// No node failure here, to expect request with the correct state to come
	const FString* FoundState = InRequest.QueryParams.Find(TEXT("state"));
	return FoundState && *FoundState == SharedData.State;
}

bool UDataLinkOAuthDefaultSettings::BuildExchangeCodeTokenUrl(FUrlBuilder& OutRequestUrl, FDataLinkNodeOAuthInstance& InOAuthInstance, FStringView InAuthCode) const
{
	const FString RedirectUriWithPort = FString::Printf(TEXT("%s:%s"), LoopbackAddress, *FString::FromInt(InOAuthInstance.ListenPort));

	OutRequestUrl << TokenExchangeEndpoint
		<< TEXT("?grant_type=authorization_code")
		<< TEXT("&redirect_uri=") << FPlatformHttp::UrlEncode(RedirectUriWithPort)
		<< TEXT("&client_id=") << ClientId
		<< TEXT("&client_secret=") << ClientSecret
		<< TEXT("&code=") << InAuthCode;

	return true;
}
