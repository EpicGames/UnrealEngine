// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthHandle.h"
#include <atomic>

namespace UE::DataLinkOAuth::Private
{
	std::atomic<uint64> GNextId(0);

	uint64 GenerateNewId()
	{
		uint64 NewId = ++GNextId;

		// 0 is reserved for non-initialized state
		if (NewId == 0)
		{
			NewId = ++GNextId;
		}

		return NewId;
	}
}

FDataLinkOAuthHandle FDataLinkOAuthHandle::GenerateHandle()
{
	FDataLinkOAuthHandle Handle;
	Handle.Id = UE::DataLinkOAuth::Private::GenerateNewId();
	return Handle;
}

bool FDataLinkOAuthHandle::IsValid() const
{
	return Id != 0;
}

void FDataLinkOAuthHandle::Reset()
{
	Id = 0;
}
