// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthInstance.h"
#include "DataLinkOAuthSubsystem.h"

FDataLinkNodeOAuthInstance::~FDataLinkNodeOAuthInstance()
{
	StopListening();
}

void FDataLinkNodeOAuthInstance::StopListening()
{
	if (ListenHandle.IsValid())
	{
		if (UDataLinkOAuthSubsystem* const OAuthSubsystem = UDataLinkOAuthSubsystem::Get())
        {
        	OAuthSubsystem->UnregisterListenInstance(ListenHandle);
        }
	}
	ListenHandle.Reset();	
}
