// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthLog.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogDataLinkOAuth);

IMPLEMENT_MODULE(FDefaultModuleImpl, DataLinkOAuth)
