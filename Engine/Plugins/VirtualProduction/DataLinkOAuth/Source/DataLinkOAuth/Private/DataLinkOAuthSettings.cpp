// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthSettings.h"
#include "DataLinkHttpSettings.h"
#include "DataLinkOAuthLog.h"
#include "DataLinkOAuthToken.h"
#include "DataLinkOAuthUtils.h"
#include "Dom/JsonObject.h"
#include "HttpServerRequest.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"

FInstancedStruct UDataLinkOAuthSettings::MakeSharedData() const
{
	FInstancedStruct InstanceData;
	InstanceData.InitializeAs(SharedDataType);
	return InstanceData;
}

bool UDataLinkOAuthSettings::FindAuthCode(const FHttpServerRequest& InRequest, FDataLinkNodeOAuthInstance& InOAuthInstance, FStringView& OutAuthCodeView) const
{
	if (const FString* FoundCode = InRequest.QueryParams.Find(TEXT("code")))
	{
		OutAuthCodeView = *FoundCode;
		return true;
	}
	return false;
}

bool UDataLinkOAuthSettings::BuildAuthToken(FStringView InAccessResponse, FDataLinkOAuthToken& OutAuthToken) const
{
	const TSharedPtr<FJsonObject> ResponseJson = UE::DataLinkOAuth::ResponseStringToJsonObject(InAccessResponse);
	if (!ResponseJson.IsValid())
	{
		UE_LOGF(LogDataLinkOAuth, Error, "Response %ls could not be converted to a Json Object", InAccessResponse.GetData());
		return false;
	}

	if (!ResponseJson->TryGetStringField(TEXT("token_type"), OutAuthToken.TokenType))
	{
		UE_LOGF(LogDataLinkOAuth, Error, "Response %ls did not have a valid 'token_type' entry", InAccessResponse.GetData());
		return false;
	}

	if (!ResponseJson->TryGetStringField(TEXT("access_token"), OutAuthToken.AccessToken))
	{
		UE_LOGF(LogDataLinkOAuth, Error, "Response %ls did not have a valid 'access_token' entry", InAccessResponse.GetData());
		return false;
	}

	if (!ResponseJson->TryGetStringField(TEXT("refresh_token"), OutAuthToken.RefreshToken))
	{
		UE_LOGF(LogDataLinkOAuth, Warning, "Response %ls did not have a valid 'refresh_token' entry", InAccessResponse.GetData());
	}

	uint32 ExpiresInSeconds = 0;
	if (ResponseJson->TryGetNumberField(TEXT("expires_in"), ExpiresInSeconds))
	{
		OutAuthToken.ExpirationDate = FDateTime::UtcNow() + FTimespan(0, 0, ExpiresInSeconds);
	}
	else
	{
		UE_LOGF(LogDataLinkOAuth, Warning, "Response %ls did not have a valid 'expires_in' entry", InAccessResponse.GetData());
	}

	return true;
}

bool UDataLinkOAuthSettings::AuthorizeHttpRequest(const FDataLinkOAuthToken& InAuthToken, FDataLinkHttpSettings& InOutHttpSettings) const
{
	InOutHttpSettings.Headers.Add(TEXT("Authorization"), FString::Printf(TEXT("%s %s"), *InAuthToken.TokenType, *InAuthToken.AccessToken));
	return true;
}
