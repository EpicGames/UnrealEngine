// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthSubsystem.h"
#include "DataLinkOAuthSettings.h"
#include "Engine/Engine.h"
#include "IHttpRouter.h"

namespace UE::DataLinkOAuth::Private
{
	FDateTime GetPaddedNow()
	{
		constexpr int32 PaddingSeconds = 5;
		return FDateTime::UtcNow() + FTimespan(0, 0, PaddingSeconds);
	}
}

UDataLinkOAuthSubsystem* UDataLinkOAuthSubsystem::Get()
{
	if (!UObjectInitialized())
	{
		return nullptr;
	}

	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UDataLinkOAuthSubsystem>();
	}
	return nullptr;
}

FDataLinkOAuthHandle UDataLinkOAuthSubsystem::RegisterListenInstance(FListenInstance&& InInstance)
{
	FDataLinkOAuthHandle NewHandle = FDataLinkOAuthHandle::GenerateHandle();
	ListeningInstances.Add(NewHandle, MoveTemp(InInstance));
	return NewHandle;
}

void UDataLinkOAuthSubsystem::UnregisterListenInstance(FDataLinkOAuthHandle InHandle)
{
	FListenInstance ListenInstance;
	if (!ListeningInstances.RemoveAndCopyValue(InHandle, ListenInstance))
	{
		return;
	}

	TSharedPtr<IHttpRouter> Router = ListenInstance.RouterWeak.Pin();
	if (!Router.IsValid())
	{
		return;
	}

	Router->UnregisterRequestPreprocessor(ListenInstance.RequestPreprocessorHandle);
}

const FDataLinkOAuthToken* UDataLinkOAuthSubsystem::FindToken(const UDataLinkOAuthSettings* InOAuthSettings) const
{
	const FDateTime Now = UE::DataLinkOAuth::Private::GetPaddedNow();

	const FDataLinkOAuthToken* Token = Tokens.Find(FDataLinkOAuthTokenHandle(InOAuthSettings));
	if (Token && Now < Token->ExpirationDate)
	{
		return Token;	
	}

	return nullptr;
}

void UDataLinkOAuthSubsystem::RegisterToken(const UDataLinkOAuthSettings* InOAuthSettings, const FDataLinkOAuthToken& InToken)
{
	// Duplicate so that any changes to the OAuth Settings does not affect the cached token handle key
	const UDataLinkOAuthSettings* DuplicateOAuthSettings = DuplicateObject(InOAuthSettings, this);

	Tokens.Add(FDataLinkOAuthTokenHandle(DuplicateOAuthSettings), InToken);
}

void UDataLinkOAuthSubsystem::CleanExpiredTokens()
{
	const FDateTime Now = UE::DataLinkOAuth::Private::GetPaddedNow();

	for (decltype(Tokens)::TIterator Iter(Tokens); Iter; ++Iter)
	{
		const FDataLinkOAuthToken& Token = Iter.Value();
		if (Now >= Token.ExpirationDate)
		{
			Iter.RemoveCurrent();
		}
	}
}
