// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthTokenHandle.h"
#include "DataLinkOAuthSettings.h"
#include "Serialization/ArchiveObjectCrc32.h"

namespace UE::DataLinkOAuth::Private
{
	bool IsTokenIndifferent(const FProperty* InProperty)
	{
		return InProperty->HasAllPropertyFlags(CPF_Transient);
	}
}

FDataLinkOAuthTokenHandle::FDataLinkOAuthTokenHandle(const UDataLinkOAuthSettings* InOAuthSettings)
	: OAuthSettings(InOAuthSettings)
{
	RecalculateHash();
}

bool FDataLinkOAuthTokenHandle::operator==(const FDataLinkOAuthTokenHandle& InOther) const
{
	if (CachedHash != InOther.CachedHash)
	{
		return false;
	}

	// If both are equal (unlikely) then no need to perform deep comparison
	if (OAuthSettings == InOther.OAuthSettings)
	{
		return true;
	}

	if (!OAuthSettings || !InOther.OAuthSettings)
	{
		return false;
	}

	const UClass* const OAuthSettingsClass = OAuthSettings->GetClass();
	if (OAuthSettingsClass != InOther.OAuthSettings->GetClass())
	{
		return false;
	}

	constexpr EPropertyPortFlags PortFlags = PPF_DeepComparison;

	for (const FProperty* Property : TFieldRange<FProperty>(OAuthSettingsClass))
	{
		if (UE::DataLinkOAuth::Private::IsTokenIndifferent(Property))
		{
			continue;
		}

		for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
		{
			if (!Property->Identical_InContainer(OAuthSettings, InOther.OAuthSettings, Index, PortFlags))
			{
				return false;
			}
		}
	}

	return true;
}

void FDataLinkOAuthTokenHandle::RecalculateHash()
{
	if (!OAuthSettings)
	{
		CachedHash = 0;
		return;
	}

	class : public FArchiveObjectCrc32
	{
		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
		{
			return UE::DataLinkOAuth::Private::IsTokenIndifferent(InProperty);
		}
	} Archive;

	CachedHash = Archive.Crc32(ConstCast(OAuthSettings), 0);
}
