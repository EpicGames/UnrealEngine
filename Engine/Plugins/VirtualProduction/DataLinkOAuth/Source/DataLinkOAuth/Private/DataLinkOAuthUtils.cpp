// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthUtils.h"
#include "Containers/StringView.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

TSharedPtr<FJsonObject> UE::DataLinkOAuth::ResponseStringToJsonObject(FStringView InResponseString)
{
	TSharedPtr<FJsonValue> ResponseJson;
	if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(InResponseString.GetData()), ResponseJson))
	{
		return nullptr;
	}

	TSharedPtr<FJsonObject>* JsonObject;
	if (!ResponseJson->TryGetObject(JsonObject))
	{
		return nullptr;
	}

	return *JsonObject;
}
