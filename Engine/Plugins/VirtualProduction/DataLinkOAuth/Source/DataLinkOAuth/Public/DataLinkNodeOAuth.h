// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "DataLinkOAuthHandle.h"
#include "DataLinkNodeOAuth.generated.h"

class UDataLinkOAuthSettings;
struct FHttpServerRequest;

namespace UE::DataLinkOAuth
{
	const FLazyName InputHttp(TEXT("InputHttp"));
	const FLazyName InputOAuth(TEXT("InputOAuth"));
}

USTRUCT(BlueprintType, DisplayName="Data Link OAuth Settings")
struct FDataLinkOAuthSettingsWrapper
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link")
	TObjectPtr<UDataLinkOAuthSettings> OAuthSettings;
};

UCLASS(MinimalAPI, DisplayName="OAuth", Category="Authorization")
class UDataLinkNodeOAuth : public UDataLinkNode
{
	GENERATED_BODY()

public:
	DATALINKOAUTH_API UDataLinkNodeOAuth();

protected:
	//~ Begin UDataLinkNode
	DATALINKOAUTH_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	DATALINKOAUTH_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode

private:
	void OnAuthResponse(const FHttpServerRequest& InRequest, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const;

	void OnExchangeCodeResponse(FStringView InResponse, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const;
};
