// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkOAuthSettings.h"
#include "DataLinkOAuthDefaultSettings.generated.h"

UCLASS(MinimalAPI, DisplayName="OAuth Default Settings")
class UDataLinkOAuthDefaultSettings : public UDataLinkOAuthSettings
{
	GENERATED_BODY()

public:
	DATALINKOAUTH_API UDataLinkOAuthDefaultSettings();

	//~ Begin UDataLinkOAuthSettings
	DATALINKOAUTH_API virtual bool BuildAuthRequestUrl(FUrlBuilder& OutRequestUrl, FDataLinkNodeOAuthInstance& InOAuthInstance) const override;
	DATALINKOAUTH_API virtual bool ValidateRequest(const FHttpServerRequest& InRequest, FDataLinkNodeOAuthInstance& InOAuthInstance) const override;
	DATALINKOAUTH_API virtual bool BuildExchangeCodeTokenUrl(FUrlBuilder& OutRequestUrl, FDataLinkNodeOAuthInstance& InOAuthInstance, FStringView InAuthCode) const override;
	//~ End UDataLinkOAuthSettings

private:
	UPROPERTY(EditAnywhere, Category="OAuth")
	FString AuthorizationURL;

	UPROPERTY(EditAnywhere, Category="OAuth")
	FString TokenExchangeEndpoint;

	UPROPERTY(EditAnywhere, Category="OAuth")
	TArray<FString> Scopes;
};
