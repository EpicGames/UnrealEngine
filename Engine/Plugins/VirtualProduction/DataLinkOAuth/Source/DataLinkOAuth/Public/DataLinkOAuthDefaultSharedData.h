// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DataLinkOAuthDefaultSharedData.generated.h"

USTRUCT()
struct FDataLinkOAuthDefaultSharedData
{
	GENERATED_BODY()

	DATALINKOAUTH_API FDataLinkOAuthDefaultSharedData();

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	FString State;
};
