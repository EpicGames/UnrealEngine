// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/TypeHash.h"

class FDataLinkOAuthHandle
{
public:
	FDataLinkOAuthHandle() = default;

	static FDataLinkOAuthHandle GenerateHandle();

	/** Whether this handle was ever initialized */
	DATALINKOAUTH_API bool IsValid() const;

	/** Clear handle to indicate it is no longer used */
	DATALINKOAUTH_API void Reset();

	bool operator==(const FDataLinkOAuthHandle& InOther) const
	{
		return Id == InOther.Id;
	}

	friend uint32 GetTypeHash(const FDataLinkOAuthHandle& InHandle)
	{
		return GetTypeHash(InHandle.Id);
	}

private:
	uint64 Id = 0;
};
