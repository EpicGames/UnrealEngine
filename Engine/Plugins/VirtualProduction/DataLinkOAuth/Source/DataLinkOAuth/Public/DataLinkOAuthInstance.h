// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkOAuthHandle.h"
#include "StructUtils/InstancedStruct.h"
#include "DataLinkOAuthInstance.generated.h"

USTRUCT()
struct FDataLinkNodeOAuthInstance
{
	GENERATED_BODY()

	DATALINKOAUTH_API ~FDataLinkNodeOAuthInstance();

	void StopListening();

	FDataLinkOAuthHandle ListenHandle;

	UPROPERTY()
	int32 ListenPort = 0;

	UPROPERTY()
	FInstancedStruct SharedData;
};
