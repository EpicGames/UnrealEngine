// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "UObject/Object.h"
#include "DataLinkOAuthSettings.generated.h"

struct FDataLinkHttpSettings;
struct FDataLinkNodeOAuthInstance;
struct FDataLinkOAuthToken;
struct FHttpServerRequest;
struct FInstancedStruct;

UCLASS(MinimalAPI, Abstract, DisplayName="Motion Design Data Link OAuth Settings")
class UDataLinkOAuthSettings : public UObject
{
	GENERATED_BODY()

public:
	using FUrlBuilder = TStringBuilder<512>;

	/**
	 * Instantiated data of type that matches the 'SharedDataType'.
	 * This is data that will live throughout the OAuth process and used by stages to read from/write to
	 */
	FInstancedStruct MakeSharedData() const;

	/**
	 * Builds an Authorization URL
	 * @param OutRequestUrl the resulting request URL
	 * @param InOAuthInstance oauth instance data allowing different stages of OAuth to read from/write to
	 * @return true if the request URL was successfully created
	 */
	virtual bool BuildAuthRequestUrl(FUrlBuilder& OutRequestUrl, FDataLinkNodeOAuthInstance& InOAuthInstance) const
	{
		return false;
	}

	/**
	 * Validates that the request relates to our OAuth flow
	 * @param InRequest the request containing data to validate against
	 * @param InOAuthInstance oauth instance data containing data to use for validation
	 * @return true if request is valid. False if it's a request to ignore
	 */
	virtual bool ValidateRequest(const FHttpServerRequest& InRequest, FDataLinkNodeOAuthInstance& InOAuthInstance) const
	{
		return true;
	}

	/**
	 * Attempts to find the Auth Code within the given Request
	 * @param InRequest the request potentially containing the authorization code to retrieve
	 * @param InOAuthInstance oauth instance data allowing different stages of OAuth to read from/write to
	 * @param OutAuthCodeView the auth code string view if it was found
	 * @return true if the Auth Code was found
	 */
	DATALINKOAUTH_API virtual bool FindAuthCode(const FHttpServerRequest& InRequest, FDataLinkNodeOAuthInstance& InOAuthInstance, FStringView& OutAuthCodeView) const;

	/**
	 * Builds an Exchange Code for Access Token URL
	 * @param OutRequestUrl the resulting request url
	 * @param InOAuthInstance oauth instance data allowing different stages of OAuth to read from/write to
	 * @param InAuthCode the authorization code to exchange
	 * @return true if the Request URL was successfully created
	 */
	virtual bool BuildExchangeCodeTokenUrl(FUrlBuilder& OutRequestUrl, FDataLinkNodeOAuthInstance& InOAuthInstance, FStringView InAuthCode) const
	{
		return false;
	}

	/**
	 * Builds an Auth Token based on the Access Response String
	 * @param InAccessResponse the access response
	 * @param OutAuthToken the auth token to build
	 * @return true if the token was built
	 */
	DATALINKOAUTH_API virtual bool BuildAuthToken(FStringView InAccessResponse, FDataLinkOAuthToken& OutAuthToken) const;

	/**
	 * Adds Authorization header to the given Http Settings if the Access Response is valid
	 * @param InAuthToken the auth token containing the authorization parameters
	 * @param InOutHttpSettings the http settings to add authorization to
	 * @return true if the authorization was added
	 */
	DATALINKOAUTH_API virtual bool AuthorizeHttpRequest(const FDataLinkOAuthToken& InAuthToken, FDataLinkHttpSettings& InOutHttpSettings) const;

protected:
	static constexpr const TCHAR* LoopbackAddress = TEXT("http://127.0.0.1");

	/** Optional shared data type to use across the different stages of OAuth */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> SharedDataType;

	UPROPERTY(EditAnywhere, Category="Client")
	FString ClientId;

	UPROPERTY(EditAnywhere, Category="Client")
	FString ClientSecret;
};
