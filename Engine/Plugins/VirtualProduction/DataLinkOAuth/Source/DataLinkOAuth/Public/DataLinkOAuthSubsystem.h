// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "DataLinkOAuthHandle.h"
#include "DataLinkOAuthToken.h"
#include "DataLinkOAuthTokenHandle.h"
#include "Subsystems/EngineSubsystem.h"
#include "DataLinkOAuthSubsystem.generated.h"

class IHttpRouter;
class UDataLinkOAuthSettings;

UCLASS(MinimalAPI)
class UDataLinkOAuthSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	static UDataLinkOAuthSubsystem* Get();

	struct FListenInstance
	{
		TWeakPtr<IHttpRouter> RouterWeak;
		FDelegateHandle RequestPreprocessorHandle;
	};
	FDataLinkOAuthHandle RegisterListenInstance(FListenInstance&& InInstance);

	void UnregisterListenInstance(FDataLinkOAuthHandle InHandle);

	const FDataLinkOAuthToken* FindToken(const UDataLinkOAuthSettings* InOAuthSettings) const;

	void RegisterToken(const UDataLinkOAuthSettings* InOAuthSettings, const FDataLinkOAuthToken& InToken);

	void CleanExpiredTokens();

private:
	TMap<FDataLinkOAuthHandle, FListenInstance> ListeningInstances;

	UPROPERTY()
	TMap<FDataLinkOAuthTokenHandle, FDataLinkOAuthToken> Tokens;
};
