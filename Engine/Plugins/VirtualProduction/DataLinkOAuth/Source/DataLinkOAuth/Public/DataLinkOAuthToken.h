// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "DataLinkOAuthToken.generated.h"

USTRUCT()
struct FDataLinkOAuthToken
{
	GENERATED_BODY()

	UPROPERTY()
	FString TokenType;

	UPROPERTY()
	FString AccessToken;

	UPROPERTY()
	FString RefreshToken;

	UPROPERTY()
	FDateTime ExpirationDate;
};
