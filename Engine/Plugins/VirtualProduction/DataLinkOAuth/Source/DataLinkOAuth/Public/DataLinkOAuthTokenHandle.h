// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkOAuthTokenHandle.generated.h"

class UDataLinkOAuthSettings;

USTRUCT()
struct FDataLinkOAuthTokenHandle
{
	GENERATED_BODY()

	FDataLinkOAuthTokenHandle() = default;

	explicit FDataLinkOAuthTokenHandle(const UDataLinkOAuthSettings* InOAuthSettings);

	bool operator==(const FDataLinkOAuthTokenHandle& InOther) const;

	friend uint32 GetTypeHash(const FDataLinkOAuthTokenHandle& InHandle)
	{
		return InHandle.CachedHash;
	}

private:
	void RecalculateHash();

	UPROPERTY()
	uint32 CachedHash = 0;

	UPROPERTY()
	TObjectPtr<const UDataLinkOAuthSettings> OAuthSettings;
};
