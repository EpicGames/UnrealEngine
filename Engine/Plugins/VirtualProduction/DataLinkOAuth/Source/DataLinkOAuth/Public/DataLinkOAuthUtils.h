// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Templates/SharedPointerFwd.h"

class FJsonObject;

namespace UE::DataLinkOAuth
{
	DATALINKOAUTH_API TSharedPtr<FJsonObject> ResponseStringToJsonObject(FStringView InResponseString);
}
