// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLinkOAuthEditor : ModuleRules
{
    public DataLinkOAuthEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AssetDefinition",
                "AssetTools",
                "Core",
                "CoreUObject",
                "ClassViewer",
                "DataLinkOAuth",
                "Engine",
                "UnrealEd",
            }
        );
    }
}