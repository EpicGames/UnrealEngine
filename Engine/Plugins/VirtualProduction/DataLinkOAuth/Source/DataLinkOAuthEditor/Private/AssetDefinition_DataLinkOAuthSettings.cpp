// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DataLinkOAuthSettings.h"
#include "DataLinkOAuthSettings.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_DataLinkOAuthSettings"

FText UAssetDefinition_DataLinkOAuthSettings::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Data Link OAuth Settings");
}

FLinearColor UAssetDefinition_DataLinkOAuthSettings::GetAssetColor() const
{
	return FLinearColor(FColor(30,144,255));
}

TSoftClassPtr<UObject> UAssetDefinition_DataLinkOAuthSettings::GetAssetClass() const
{
	return UDataLinkOAuthSettings::StaticClass();
}

#undef LOCTEXT_NAMESPACE
