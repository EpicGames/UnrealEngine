// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOAuthSettingsFactory.h"
#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "DataLinkOAuthSettings.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "DataLinkOAuthSettingsFactory"

namespace UE::DataLinkOAuthEditor::Private
{
	class FAssetClassParentFilter : public IClassViewerFilter
	{
	public:
		/** All children of these classes will be included unless filtered out by another setting. */
		TSet<const UClass*> AllowedChildrenOfClasses;

		/** Disallowed class flags. */
		EClassFlags DisallowedClassFlags = CLASS_None;

		//~ Begin IClassViewerFilter
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFunctions) override
		{
			return !InClass->HasAnyClassFlags(DisallowedClassFlags)
				&& InFilterFunctions->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
		}
		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFunctions) override
		{
			return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
				&& InFilterFunctions->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
		}
		//~ End IClassViewerFilter
	};
}

UDataLinkOAuthSettingsFactory::UDataLinkOAuthSettingsFactory()
{
	SupportedClass = UDataLinkOAuthSettings::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

FText UDataLinkOAuthSettingsFactory::GetDisplayName() const
{
	return SupportedClass ? SupportedClass->GetDisplayNameText() : Super::GetDisplayName();
}

FString UDataLinkOAuthSettingsFactory::GetDefaultNewAssetName() const
{
	// Short name removing "Motion Design" and "DataLink" prefix for new assets
	return TEXT("NewOAuthSettings");
}

uint32 UDataLinkOAuthSettingsFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory("MotionDesignCategory");
}

bool UDataLinkOAuthSettingsFactory::ConfigureProperties()
{
	using namespace UE::DataLinkOAuthEditor;

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	const TSharedRef<Private::FAssetClassParentFilter> Filter = MakeShared<Private::FAssetClassParentFilter>();
	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown;
	Filter->AllowedChildrenOfClasses.Add(UDataLinkOAuthSettings::StaticClass());

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.ClassFilters.Add(Filter);

	const FText TitleText = LOCTEXT("OAuthSettingsPickerTitle", "Pick OAuth Settings Class");

	OAuthSettingsClass = nullptr;

	UClass* ChosenClass = nullptr;
	if (SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UDataLinkOAuthSettings::StaticClass()))
	{
		OAuthSettingsClass = ChosenClass;
		return true;
	}

	return false;
}

UObject* UDataLinkOAuthSettingsFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	if (OAuthSettingsClass)
	{
		return NewObject<UDataLinkOAuthSettings>(InParent, OAuthSettingsClass, InName, InFlags);
	}

	if (ensureAlways(InClass && InClass->IsChildOf<UDataLinkOAuthSettings>()))
	{
		return NewObject<UDataLinkOAuthSettings>(InParent, InClass, InName, InFlags);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
