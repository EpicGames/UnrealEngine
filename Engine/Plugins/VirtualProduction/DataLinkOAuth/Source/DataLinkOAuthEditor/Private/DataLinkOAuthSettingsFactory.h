// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "DataLinkOAuthSettingsFactory.generated.h"

class UDataLinkOAuthSettings;

UCLASS()
class UDataLinkOAuthSettingsFactory : public UFactory
{
	GENERATED_BODY()

public:
	UDataLinkOAuthSettingsFactory();

	//~ Begin UFactory
	virtual FText GetDisplayName() const override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
	//~ End UFactory

private:
	/** The OAuth Settings Type to create */
	UPROPERTY(EditAnywhere, Category="OAuth")
	TSubclassOf<UDataLinkOAuthSettings> OAuthSettingsClass;
};
